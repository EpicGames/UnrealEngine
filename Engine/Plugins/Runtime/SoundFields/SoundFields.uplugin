{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "SoundFields",
	"Description": "Plugin featuring a variety of basic audio SoundFields solutions.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "SoundFields",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	]
}