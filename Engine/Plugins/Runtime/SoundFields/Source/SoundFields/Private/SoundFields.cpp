// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundFields.h"
#include "AudioAnalytics.h"
#include "DSP/FloatArrayMath.h"
#include "SoundFieldRendering.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundFields)

static int32 VirtualIntermediateChannelsCvar = 1;
FAutoConsoleVariableRef CVarVirtualIntermediateChannels(
	TEXT("au.Ambisonics.VirtualIntermediateChannels"),
	VirtualIntermediateChannelsCvar,
	TEXT("Enables decoding to a virtual 7.1 speaker config before mixdown.\n")
	TEXT("0: Decode directly to output device configuration, 1: Enabled"),
	ECVF_Default);

class FAmbisonicsSoundfieldEncoder : public ISoundfieldEncoderStream
{
private:
	FSoundFieldEncoder Encoder;

public:
	virtual ~FAmbisonicsSoundfieldEncoder() {};


	virtual void Encode(const FSoundfieldEncoderInputData& InputData, ISoundfieldAudioPacket& OutputData) override
	{
		OutputData.Reset();
		EncodeAndMixIn(InputData, OutputData);
	}


	virtual void EncodeAndMixIn(const FSoundfieldEncoderInputData& InputData, ISoundfieldAudioPacket& OutputData) override
	{
		const Audio::AlignedFloatBuffer& InputAudio = InputData.AudioBuffer;
		const FSoundfieldSpeakerPositionalData& PositionalData = InputData.PositionalData;
		const FAmbisonicsSoundfieldSettings& Settings = DowncastSoundfieldRef<const FAmbisonicsSoundfieldSettings>(InputData.InputSettings);
		FAmbisonicsSoundfieldBuffer& Output = DowncastSoundfieldRef<FAmbisonicsSoundfieldBuffer>(OutputData);

		Encoder.EncodeAudioDirectlyFromOutputPositions(InputAudio, PositionalData, Settings, Output);
	}
};

class FAmbisonicsSoundfieldDecoder : public ISoundfieldDecoderStream
{
private:
	FSoundFieldDecoder Decoder;
	FQuat PreviousRotation = FQuat::Identity;

	FAmbisonicsSoundfieldBuffer RotatedAudio;
	Audio::FAlignedFloatBuffer DecoderOutputBuffer;

public:
	virtual ~FAmbisonicsSoundfieldDecoder() {};

	virtual void Decode(const FSoundfieldDecoderInputData& InputData, FSoundfieldDecoderOutputData& OutputData) override
	{
		const FAmbisonicsSoundfieldBuffer& InputAudio = DowncastSoundfieldRef<const FAmbisonicsSoundfieldBuffer>(InputData.SoundfieldBuffer);
		const FSoundfieldSpeakerPositionalData& OutputPositions = InputData.PositionalData;
		Audio::AlignedFloatBuffer& OutputAudio = OutputData.AudioBuffer;

		const FAmbisonicsSoundfieldBuffer* AudioToDecode = &InputAudio;

		if (VirtualIntermediateChannelsCvar)
		{
			Decoder.DecodeAudioToSevenOneAndDownmixToDevice(*AudioToDecode, OutputPositions, OutputAudio);
		}
		else
		{
			Decoder.DecodeAudioDirectlyToDeviceOutputPositions(*AudioToDecode, OutputPositions, OutputAudio);
		}
	}


	virtual void DecodeAndMixIn(const FSoundfieldDecoderInputData& InputData, FSoundfieldDecoderOutputData& OutputData) override
	{
		const FAmbisonicsSoundfieldBuffer& InputAudio = DowncastSoundfieldRef<const FAmbisonicsSoundfieldBuffer>(InputData.SoundfieldBuffer);
		const FSoundfieldSpeakerPositionalData& OutputPositions = InputData.PositionalData;
		Audio::AlignedFloatBuffer& OutputAudio = OutputData.AudioBuffer;

		const FAmbisonicsSoundfieldBuffer* AudioToDecode = &InputAudio;

		DecoderOutputBuffer.Reset();
		DecoderOutputBuffer.AddUninitialized(OutputData.AudioBuffer.Num());

		if (VirtualIntermediateChannelsCvar)
		{
			Decoder.DecodeAudioToSevenOneAndDownmixToDevice(*AudioToDecode, OutputPositions, DecoderOutputBuffer);
		}
		else
		{
			Decoder.DecodeAudioDirectlyToDeviceOutputPositions(*AudioToDecode, OutputPositions, DecoderOutputBuffer);
		}

		Audio::ArrayMixIn(DecoderOutputBuffer, OutputAudio);
	}
};

/**
 * Class that handles converting an ambisonics stream to a higher or lower order.
 * Currently, this class simply adds zeroed channels when increasing order,
 * and removes channels when decreasing order.
 */
class FAmbisonicsOrderConverter : public ISoundfieldTranscodeStream
{
public:

	virtual void Transcode(const ISoundfieldAudioPacket& InputData, const ISoundfieldEncodingSettingsProxy& InputSettings, ISoundfieldAudioPacket& OutputData, const ISoundfieldEncodingSettingsProxy& OutputSettings) override
	{
		const FAmbisonicsSoundfieldBuffer& InputAudio = DowncastSoundfieldRef<const FAmbisonicsSoundfieldBuffer>(InputData);
		const FAmbisonicsSoundfieldSettings& InAudioSettings = DowncastSoundfieldRef<const FAmbisonicsSoundfieldSettings>(InputSettings);

		FAmbisonicsSoundfieldBuffer& OutputAudio = DowncastSoundfieldRef<FAmbisonicsSoundfieldBuffer>(OutputData);
		const FAmbisonicsSoundfieldSettings& OutAudioSettings = DowncastSoundfieldRef<const FAmbisonicsSoundfieldSettings>(OutputSettings);

		// For ambisonics, the number of channels is equal to (N + 1)^2, where N is the order.
		const int32 InNumChannels = (InAudioSettings.Order + 1) * (InAudioSettings.Order + 1);
		const int32 OutNumChannels = (OutAudioSettings.Order + 1) * (OutAudioSettings.Order + 1);
		check(InputAudio.NumChannels == InNumChannels);
		
		const int32 InNumFrames = InputAudio.AudioBuffer.Num() / InputAudio.NumChannels;
		OutputAudio.Reset();
		OutputAudio.NumChannels = OutNumChannels;
		OutputAudio.AudioBuffer.Reset();
		OutputAudio.AudioBuffer.AddZeroed(InNumFrames * OutNumChannels);

		const int32 NumChannelsToCopy = FMath::Min(InNumChannels, OutNumChannels);

		for (int32 FrameIndex = 0; FrameIndex < InNumFrames; FrameIndex++)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannelsToCopy; ChannelIndex++)
			{
				const int32 InputIndex = FrameIndex * InNumChannels + ChannelIndex;
				const int32 OutputIndex = FrameIndex * OutNumChannels + ChannelIndex;
				OutputAudio.AudioBuffer[OutputIndex] = InputAudio.AudioBuffer[InputIndex];
			}
		}
	}


	virtual void TranscodeAndMixIn(const ISoundfieldAudioPacket& InputData, const ISoundfieldEncodingSettingsProxy& InputSettings, ISoundfieldAudioPacket& PacketToSumTo, const ISoundfieldEncodingSettingsProxy& OutputSettings) override
	{
		const FAmbisonicsSoundfieldBuffer& InputAudio = DowncastSoundfieldRef<const FAmbisonicsSoundfieldBuffer>(InputData);
		const FAmbisonicsSoundfieldSettings& InAudioSettings = DowncastSoundfieldRef<const FAmbisonicsSoundfieldSettings>(InputSettings);

		FAmbisonicsSoundfieldBuffer& OutputAudio = DowncastSoundfieldRef<FAmbisonicsSoundfieldBuffer>(PacketToSumTo);
		const FAmbisonicsSoundfieldSettings& OutAudioSettings = DowncastSoundfieldRef<const FAmbisonicsSoundfieldSettings>(OutputSettings);

		// For ambisonics, the number of channels is equal to (N + 1)^2, where N is the order.
		const int32 InNumChannels = (InAudioSettings.Order + 1) * (InAudioSettings.Order + 1);
		const int32 OutNumChannels = (OutAudioSettings.Order + 1) * (OutAudioSettings.Order + 1);
		check(InputAudio.NumChannels == InNumChannels);
		check(OutputAudio.NumChannels == OutNumChannels);

		// Since we are being asked to mix InputAudio into OutputAudio, they should have the same number of frames.
		const int32 InNumFrames = InputAudio.AudioBuffer.Num() / InputAudio.NumChannels;
		const int32 OutNumFrames = OutputAudio.AudioBuffer.Num() / OutputAudio.NumChannels;
		check(InNumFrames == OutNumFrames);

		const int32 NumChannelsToCopy = FMath::Min(InNumChannels, OutNumChannels);

		// Sum InputAudio directly into the corresponding channels in OutputAudio.
		for (int32 FrameIndex = 0; FrameIndex < InNumFrames; FrameIndex++)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannelsToCopy; ChannelIndex++)
			{
				const int32 InputIndex = FrameIndex * InNumChannels + ChannelIndex;
				const int32 OutputIndex = FrameIndex * OutNumChannels + ChannelIndex;
				OutputAudio.AudioBuffer[OutputIndex] += InputAudio.AudioBuffer[InputIndex];
			}
		}
	}

};

/**
 * Class that sums an ambisonics stream into an output stream, rotating the ambisonics stream as necessary to get the ambisonics field in world coordinates.
 */
class FAmbisonicsMixer : public ISoundfieldMixerStream
{
private:
	// This is a temp buffer that we use to rotate InputData in MixTogether to the world rotation.
	FAmbisonicsSoundfieldBuffer RotatedAudio;
public:
	FAmbisonicsMixer() {}

	virtual void MixTogether(const FSoundfieldMixerInputData& InputData, ISoundfieldAudioPacket& PacketToSumTo) override
	{
		const FAmbisonicsSoundfieldBuffer& InAudio = DowncastSoundfieldRef<const FAmbisonicsSoundfieldBuffer>(InputData.InputPacket);
		FAmbisonicsSoundfieldBuffer& OutAudio = DowncastSoundfieldRef<FAmbisonicsSoundfieldBuffer>(PacketToSumTo);

		// Lazily initialize our output and rotated packet.
		if (OutAudio.NumChannels == 0)
		{
			OutAudio.NumChannels = InAudio.NumChannels;
			OutAudio.AudioBuffer.Reset();
			OutAudio.AudioBuffer.AddZeroed(InAudio.AudioBuffer.Num());
		}

		if (RotatedAudio.NumChannels == 0)
		{
			RotatedAudio.NumChannels = InAudio.NumChannels;
			RotatedAudio.AudioBuffer.Reset();
			RotatedAudio.AudioBuffer.AddZeroed(InAudio.AudioBuffer.Num());
		}

		check(OutAudio.NumChannels == InAudio.NumChannels);
		check(InAudio.AudioBuffer.Num() == OutAudio.AudioBuffer.Num());

		const FQuat AmountToRotateBy = InAudio.Rotation.Inverse();
		const FQuat PreviousAmountRotatedBy = InAudio.PreviousRotation.Inverse();
		FSoundFieldDecoder::RotateFirstOrderAmbisonicsBed(InAudio, RotatedAudio, AmountToRotateBy, PreviousAmountRotatedBy);
		Audio::ArrayMixIn(RotatedAudio.AudioBuffer, OutAudio.AudioBuffer, InputData.SendLevel);
	}
};

FAmbisonicsSoundfieldFormat::FAmbisonicsSoundfieldFormat()
{
	ISoundfieldFactory::RegisterSoundfieldFormat(this);
}

FAmbisonicsSoundfieldFormat::~FAmbisonicsSoundfieldFormat()
{
	ISoundfieldFactory::UnregisterSoundfieldFormat(this);
}

FName FAmbisonicsSoundfieldFormat::GetSoundfieldFormatName()
{
	return GetUnrealAmbisonicsFormatName();
}

TUniquePtr<ISoundfieldEncoderStream> FAmbisonicsSoundfieldFormat::CreateEncoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings)
{
	return TUniquePtr<ISoundfieldEncoderStream>(new FAmbisonicsSoundfieldEncoder());
}

TUniquePtr<ISoundfieldDecoderStream> FAmbisonicsSoundfieldFormat::CreateDecoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings)
{
	return TUniquePtr<ISoundfieldDecoderStream>(new FAmbisonicsSoundfieldDecoder());
}

TUniquePtr<ISoundfieldTranscodeStream> FAmbisonicsSoundfieldFormat::CreateTranscoderStream(const FName SourceFormat, const ISoundfieldEncodingSettingsProxy& InitialSourceSettings, const FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& InitialDestinationSettings, const FAudioPluginInitializationParams& InitInfo)
{
	// Currently we only support transcoding from other ambisonics streams with the same format.
	check(SourceFormat == GetSoundfieldFormatName());
	check(DestinationFormat == GetSoundfieldFormatName());
	return TUniquePtr<ISoundfieldTranscodeStream>(new FAmbisonicsOrderConverter());
}

TUniquePtr<ISoundfieldMixerStream> FAmbisonicsSoundfieldFormat::CreateMixerStream(const ISoundfieldEncodingSettingsProxy& InitialSettings)
{
	Audio::Analytics::RecordEvent_Usage("SoundFields.MixerStreamCreated");
	return TUniquePtr<ISoundfieldMixerStream>(new FAmbisonicsMixer());
}

TUniquePtr<ISoundfieldAudioPacket> FAmbisonicsSoundfieldFormat::CreateEmptyPacket()
{
	return TUniquePtr<ISoundfieldAudioPacket>(new FAmbisonicsSoundfieldBuffer());
}

bool FAmbisonicsSoundfieldFormat::IsTranscodeRequiredBetweenSettings(const ISoundfieldEncodingSettingsProxy& SourceSettings, const ISoundfieldEncodingSettingsProxy& DestinationSettings)
{
	const FAmbisonicsSoundfieldSettings& SourceAmbiSettings = DowncastSoundfieldRef<const FAmbisonicsSoundfieldSettings>(SourceSettings);
	const FAmbisonicsSoundfieldSettings& DestAmbiSettings = DowncastSoundfieldRef<const FAmbisonicsSoundfieldSettings>(DestinationSettings);

	// We only need to transcode between different orders of ambisonics.
	return SourceAmbiSettings.Order != DestAmbiSettings.Order;
}

bool FAmbisonicsSoundfieldFormat::CanTranscodeFromSoundfieldFormat(FName InputFormat, const ISoundfieldEncodingSettingsProxy& InputEncodingSettings)
{
	return false;
}

UClass* FAmbisonicsSoundfieldFormat::GetCustomEncodingSettingsClass() const
{
	return UAmbisonicsEncodingSettings::StaticClass();
}

USoundfieldEncodingSettingsBase* FAmbisonicsSoundfieldFormat::GetDefaultEncodingSettings()
{
	check(IsInGameThread() || IsInAudioThread());
	return GetMutableDefault<UAmbisonicsEncodingSettings>();
}

bool FAmbisonicsSoundfieldFormat::CanTranscodeToSoundfieldFormat(FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& DestinationEncodingSettings)
{
	return false;
}


TUniquePtr<ISoundfieldEncodingSettingsProxy> UAmbisonicsEncodingSettings::GetProxy() const
{
	FAmbisonicsSoundfieldSettings* Proxy = new FAmbisonicsSoundfieldSettings();
	Proxy->Order = FMath::Clamp(AmbisonicsOrder, 1, 5);
	return TUniquePtr<ISoundfieldEncodingSettingsProxy>(Proxy);
}

