// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundFieldsModule.h"
#include "Modules/ModuleManager.h"


void FSoundFieldsModule::StartupModule()
{
}

void FSoundFieldsModule::ShutdownModule()
{
	// Nothing done here for now.
}

IMPLEMENT_MODULE(FSoundFieldsModule, SoundFields)
