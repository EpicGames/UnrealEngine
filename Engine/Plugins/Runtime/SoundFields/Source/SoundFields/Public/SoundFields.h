// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISoundfieldFormat.h"
#include "SoundFields.generated.h"

#define UE_API SOUNDFIELDS_API

UCLASS(MinimalAPI, config = Engine, editinlinenew, BlueprintType)
class UAmbisonicsEncodingSettings : public USoundfieldEncodingSettingsBase
{
	GENERATED_BODY()

public:

protected:
	UPROPERTY(EditAnywhere, Category = EncodingSettings, meta=(ClampMin="1", ClampMax="5", UIMin="1", UIMax="5"))
	int32 AmbisonicsOrder;

	UE_API virtual TUniquePtr<ISoundfieldEncodingSettingsProxy> GetProxy() const override;

	friend class FAmbisonicsSoundfieldFormat;
};

// This is the default format for Ambisonics in the Unreal Audio Engine.
class FAmbisonicsSoundfieldFormat : public ISoundfieldFactory
{
public:
	FAmbisonicsSoundfieldFormat();

	// dtor
	virtual ~FAmbisonicsSoundfieldFormat();

	// Begin ISoundfieldFactory
	virtual FName GetSoundfieldFormatName() override;
	virtual TUniquePtr<ISoundfieldEncoderStream> CreateEncoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings) override;
	virtual TUniquePtr<ISoundfieldDecoderStream> CreateDecoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings) override;
	virtual TUniquePtr<ISoundfieldTranscodeStream> CreateTranscoderStream(const FName SourceFormat, const ISoundfieldEncodingSettingsProxy& InitialSourceSettings, const FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& InitialDestinationSettings, const FAudioPluginInitializationParams& InitInfo) override;
	virtual TUniquePtr<ISoundfieldMixerStream> CreateMixerStream(const ISoundfieldEncodingSettingsProxy& InitialSettings) override;
	virtual TUniquePtr<ISoundfieldAudioPacket> CreateEmptyPacket() override;
	virtual bool IsTranscodeRequiredBetweenSettings(const ISoundfieldEncodingSettingsProxy& SourceSettings, const ISoundfieldEncodingSettingsProxy& DestinationSettings) override;
	virtual bool CanTranscodeFromSoundfieldFormat(FName InputFormat, const ISoundfieldEncodingSettingsProxy& InputEncodingSettings) override;
	virtual UClass* GetCustomEncodingSettingsClass() const override;
	virtual USoundfieldEncodingSettingsBase* GetDefaultEncodingSettings() override;
	virtual bool CanTranscodeToSoundfieldFormat(FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& DestinationEncodingSettings) override;
	//~End ISoundfieldFactory
};

#undef UE_API
