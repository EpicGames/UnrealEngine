// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "SoundFields.h"

class FSoundFieldsModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

private:
	FAmbisonicsSoundfieldFormat SFFactory;
};
