// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;

public class SoundFields : ModuleRules
{
	public SoundFields(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			}
			);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"SignalProcessing",
				"SoundFieldRendering",
				"AudioExtensions"
			}
			);

        PrivateIncludePathModuleNames.Add("TargetPlatform");
    }
}
